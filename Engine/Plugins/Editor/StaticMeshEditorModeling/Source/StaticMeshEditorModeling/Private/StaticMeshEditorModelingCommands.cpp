// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorModelingCommands.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "StaticMeshEditorModelingCommands"

FStaticMeshEditorModelingCommands::FStaticMeshEditorModelingCommands()
	: TCommands<FStaticMeshEditorModelingCommands>(TEXT("MeshModelingTools"), NSLOCTEXT("Contexts", "MeshModelingTools", "Static Mesh Editor - Modeling Tools"), NAME_None, FAppStyle::GetAppStyleSetName())
{}

void FStaticMeshEditorModelingCommands::RegisterCommands()
{
	UI_COMMAND(ToggleStaticMeshEditorModelingMode, "Enable Mesh Modeling Tools", "Toggles Mesh Modeling Tools on or off.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

const FStaticMeshEditorModelingCommands& FStaticMeshEditorModelingCommands::Get()
{
	return TCommands<FStaticMeshEditorModelingCommands>::Get();
}

#undef LOCTEXT_NAMESPACE
