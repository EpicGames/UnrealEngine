// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FStaticMeshEditorModelingCommands : public TCommands<FStaticMeshEditorModelingCommands>
{
public:
	FStaticMeshEditorModelingCommands();
	void RegisterCommands() override;
	static const FStaticMeshEditorModelingCommands& Get();

	TSharedPtr<FUICommandInfo> ToggleStaticMeshEditorModelingMode;
};

