// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorModelingMode.h"
#include "StaticMeshEditorModelingToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ToolTargetManager.h"
#include "ModelingToolsManagerActions.h"
#include "ModelingToolsEditorModeSettings.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "Tools/GenerateStaticMeshLODAssetTool.h"
#include "Tools/LODManagerTool.h"
#include "MeshInspectorTool.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "StaticMeshEditorModelingMode"

const FEditorModeID UStaticMeshEditorModelingMode::Id("StaticMeshEditorModelingMode");

UStaticMeshEditorModelingMode::UStaticMeshEditorModelingMode()
{
	Info = FEditorModeInfo(Id, LOCTEXT("StaticMeshEditorModelingMode", "Modeling"), FSlateIcon("ModelingToolsStyle", "LevelEditor.ModelingToolsMode"), false);
}

void UStaticMeshEditorModelingMode::Enter()
{
	UEdMode::Enter();

	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UStaticMeshComponentToolTargetFactory>(GetToolManager()));

	const FModelingToolsManagerCommands& ToolManagerCommands = FModelingToolsManagerCommands::Get();

	const UModelingToolsEditorModeSettings* ModelingModeSettings = GetDefault<UModelingToolsEditorModeSettings>();

	UGenerateStaticMeshLODAssetToolBuilder* GenerateStaticMeshLODAssetToolBuilder = NewObject<UGenerateStaticMeshLODAssetToolBuilder>();
	GenerateStaticMeshLODAssetToolBuilder->bUseAssetEditorMode = true;
	GenerateStaticMeshLODAssetToolBuilder->bInRestrictiveMode = ModelingModeSettings->InRestrictiveMode();
	RegisterTool(ToolManagerCommands.BeginGenerateStaticMeshLODAssetTool, TEXT("BeginGenerateStaticMeshLODAssetTool"), GenerateStaticMeshLODAssetToolBuilder);
	RegisterTool(ToolManagerCommands.BeginLODManagerTool, TEXT("BeginLODManagerTool"), NewObject<ULODManagerToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginMeshInspectorTool, TEXT("BeginMeshInspectorTool"), NewObject<UMeshInspectorToolBuilder>());
}

bool UStaticMeshEditorModelingMode::OnRequestClose()
{
	bool bAllowClose = true;

	if (GetInteractiveToolsContext()->HasActiveTool())
	{
		UInteractiveToolsContext* ITC = GetInteractiveToolsContext();

		if (GetInteractiveToolsContext()->CanAcceptActiveTool())
		{
			EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(
				EAppMsgType::YesNoCancel,
				LOCTEXT("ShouldApplyModelingTool", "Would you like to Accept the current modeling tool changes?")
			);

			switch (YesNoCancelReply)
			{
			case EAppReturnType::Yes:
				ITC->DeactivateAllActiveTools(EToolShutdownType::Accept);
				bAllowClose = true;
				break;
			case EAppReturnType::No:
				ITC->DeactivateAllActiveTools(EToolShutdownType::Cancel);
				bAllowClose = true;
				break;
			case EAppReturnType::Cancel:
				bAllowClose = false;
				break;
			}
		}
		else
		{
			// No Accept option, so cancel and close
			ITC->DeactivateAllActiveTools(EToolShutdownType::Cancel);
			bAllowClose = true;
		}
	}

	return bAllowClose;
}

bool UStaticMeshEditorModelingMode::UsesToolkits() const
{ 
	return true; 
}

void UStaticMeshEditorModelingMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FStaticMeshEditorModelingToolkit);
}

#undef LOCTEXT_NAMESPACE
