// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorModelingModule.h"
#include "StaticMeshEditorModelingCommands.h"
#include "StaticMeshEditorModelingMode.h"
#include "ToolMenus.h"
#include "StaticMeshEditorModule.h"
#include "IStaticMeshEditor.h"
#include "EditorModeManager.h"

#define LOCTEXT_NAMESPACE "FStaticMeshEditorModelingModule"

void FStaticMeshEditorModelingModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	FStaticMeshEditorModelingCommands::Register();

	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FStaticMeshEditorModelingModule::RegisterMenusAndToolbars));

	IStaticMeshEditorModule& StaticMeshEditorModule = FModuleManager::LoadModuleChecked<IStaticMeshEditorModule>("StaticMeshEditor");
	TArray<IStaticMeshEditorModule::FStaticMeshEditorToolbarExtender>& ToolbarExtenders = StaticMeshEditorModule.GetAllStaticMeshEditorToolbarExtenders();

	// Add a button to activate modeling mode to the static mesh editor toolbar
	auto ExtendStaticMeshEditorToolbar = [this](const TSharedRef<FUICommandList> InCommandList, TSharedRef<IStaticMeshEditor> Editor) -> TSharedRef<FExtender>
	{
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
		TWeakPtr<IStaticMeshEditor> Ptr(Editor);

		InCommandList->MapAction(FStaticMeshEditorModelingCommands::Get().ToggleStaticMeshEditorModelingMode,
			FExecuteAction::CreateRaw(this, &FStaticMeshEditorModelingModule::OnToggleStaticMeshEditorModelingMode, Ptr),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FStaticMeshEditorModelingModule::IsStaticMeshEditorModelingModeActive, Ptr));

		return ToolbarExtender.ToSharedRef();
	};

	ToolbarExtenders.Add(IStaticMeshEditorModule::FStaticMeshEditorToolbarExtender::CreateLambda(ExtendStaticMeshEditorToolbar));
}


void FStaticMeshEditorModelingModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);
	FStaticMeshEditorModelingCommands::Unregister();
}

bool FStaticMeshEditorModelingModule::IsStaticMeshEditorModelingModeActive(TWeakPtr<IStaticMeshEditor> InEditor) const
{
	TSharedPtr<IStaticMeshEditor> PinnedEditor = InEditor.Pin();
	return PinnedEditor.IsValid() && PinnedEditor->GetEditorModeManager().IsModeActive(UStaticMeshEditorModelingMode::Id);
}


void FStaticMeshEditorModelingModule::OnToggleStaticMeshEditorModelingMode(TWeakPtr<IStaticMeshEditor> InEditor)
{
	TSharedPtr<IStaticMeshEditor> PinnedEditor = InEditor.Pin();
	if (PinnedEditor.IsValid())
	{
		if (!IsStaticMeshEditorModelingModeActive(InEditor))
		{
			PinnedEditor->GetEditorModeManager().ActivateMode(UStaticMeshEditorModelingMode::Id, true);
		}
		else
		{
			PinnedEditor->GetEditorModeManager().ActivateDefaultMode();
		}
	}
}

void FStaticMeshEditorModelingModule::RegisterMenusAndToolbars()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* Toolbar = UToolMenus::Get()->ExtendMenu("AssetEditor.StaticMeshEditor.ToolBar");

	{
		FToolMenuSection& Section = Toolbar->FindOrAddSection("StaticMesh");
		Section.AddDynamicEntry("ToggleStaticMeshEditorModelingMode", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
				FStaticMeshEditorModelingCommands::Get().ToggleStaticMeshEditorModelingMode,
				LOCTEXT("StaticMeshEditorModelingMode", "Modeling Tools"),
				LOCTEXT("StaticMeshEditorModelingModeTooltip", "Opens the Modeling Tools palette that provides selected mesh modification tools."),
				FSlateIcon("ModelingToolsStyle", "LevelEditor.ModelingToolsMode")));
		}));
	}
}


#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FStaticMeshEditorModelingModule, StaticMeshEditorModeling)

