// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshEditorModelingToolkit.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "ModelingToolsEditorModeStyle.h"
#include "ModelingToolsManagerActions.h"
#include "Tools/UEdMode.h"
#include "InteractiveTool.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "IDetailsView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Tools/GenerateStaticMeshLODAssetTool.h"

#define LOCTEXT_NAMESPACE "StaticMeshEditorModelingToolkit"

static const FName LODManagementTabName(TEXT("LODManagement"));

void FStaticMeshEditorModelingToolkit::Init(
	const TSharedPtr<IToolkitHost>& InToolkitHost,
	TWeakObjectPtr<UEdMode> InOwningMode )
{
	ModeWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ModeWarningArea->SetText(FText::GetEmpty());
	ModeWarningArea->SetVisibility(EVisibility::Collapsed);

	ModeHeaderArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ModeHeaderArea->SetJustification(ETextJustify::Center);

	ToolWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ToolWarningArea->SetText(FText::GetEmpty());

	SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(4)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ModeWarningArea->AsShared()
			]
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ModeHeaderArea->AsShared()
			]
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ToolWarningArea->AsShared()
			]
			+ SVerticalBox::Slot().FillHeight(1.f).HAlign(HAlign_Fill)
			[
				SAssignNew(ToolDetailsContainer, SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
			]
		];


	FModeToolkit::Init(InToolkitHost, InOwningMode);

	ToolDetailsContainer->SetContent(DetailsView.ToSharedRef());

	ClearNotification();
	ClearWarning();

	ActiveToolName = FText::GetEmpty();
	ActiveToolMessage = FText::GetEmpty();

	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolNotificationMessage.AddSP(this, &FStaticMeshEditorModelingToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolWarningMessage.AddSP(this, &FStaticMeshEditorModelingToolkit::PostWarning);


	// ViewportOverlay
	SAssignNew(ViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FStaticMeshEditorModelingToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayAccept", "Accept"))
				.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanAcceptActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayComplete", "Complete"))
				.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		]	
	];

	
	SetCurrentPalette("LODManagement");
}


TSharedPtr<SWidget> FStaticMeshEditorModelingToolkit::GetInlineContent() const
{
	return ToolkitWidget;
}

FName FStaticMeshEditorModelingToolkit::GetToolkitFName() const
{
	return FName("MeshLODEditModeToolkit");
}

FText FStaticMeshEditorModelingToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("DisplayName", "FMeshLODPluginEditModeToolkit Tool");
}


void FStaticMeshEditorModelingToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames = { LODManagementTabName };
}

FText FStaticMeshEditorModelingToolkit::GetToolPaletteDisplayName(FName PaletteName) const
{
	return FText::FromName(PaletteName);
}

void FStaticMeshEditorModelingToolkit::BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder)
{
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
	
	if (PaletteName == LODManagementTabName)
	{
		ToolbarBuilder.AddToolBarButton(Commands.BeginGenerateStaticMeshLODAssetTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginLODManagerTool);
		ToolbarBuilder.AddToolBarButton(Commands.BeginMeshInspectorTool);
	}
}

void FStaticMeshEditorModelingToolkit::PostNotification(const FText& InMessage)
{
	ActiveToolMessage = InMessage;
}


void FStaticMeshEditorModelingToolkit::ClearNotification()
{
	ActiveToolMessage = FText::GetEmpty();
}

FText FStaticMeshEditorModelingToolkit::GetActiveToolDisplayName() const
{
	return ActiveToolName;
}

FText FStaticMeshEditorModelingToolkit::GetActiveToolMessage() const
{
	return ActiveToolMessage;
}

void FStaticMeshEditorModelingToolkit::PostWarning(const FText& Message)
{
	ToolWarningArea->SetText(Message);
	ToolWarningArea->SetVisibility(EVisibility::Visible);
}


void FStaticMeshEditorModelingToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}

void FStaticMeshEditorModelingToolkit::UpdateActiveToolProperties(UInteractiveTool* Tool)
{
	if (Tool)
	{
		DetailsView->SetObjects(Tool->GetToolProperties(true));
	}
}

void FStaticMeshEditorModelingToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolStarted(Manager, Tool);

	Tool->OnPropertySetsModified.AddSP(this, &FStaticMeshEditorModelingToolkit::UpdateActiveToolProperties, Tool);

	ModeHeaderArea->SetVisibility(EVisibility::Collapsed);
	ActiveToolName = Tool->GetToolInfo().ToolDisplayName;

	FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager()->GetActiveToolName(EToolSide::Left);
	ActiveToolIdentifier.InsertAt(0, ".");
	FName ActiveToolIconName = ISlateStyle::Join(FModelingToolsManagerCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	ActiveToolIcon = FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(ActiveToolIconName);

	GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());

	const UInteractiveTool* CurrentTool = GetScriptableEditorMode()->GetToolManager()->GetActiveTool(EToolSide::Left);
	if (CurrentTool->GetClass()->IsChildOf<UGenerateStaticMeshLODAssetTool>())
	{
		PostWarning(LOCTEXT("AutoLODStaticMeshEditorWarning", "Note: This tool may create new assets including new Static Meshes, Textures, and Materials"));
	}
}


void FStaticMeshEditorModelingToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolEnded(Manager, Tool);

	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	if (Tool)
	{
		Tool->OnPropertySetsModified.RemoveAll(this);
	}

	ModeHeaderArea->SetVisibility(EVisibility::Visible);
	ActiveToolName = FText::GetEmpty();
	ActiveToolIcon = nullptr;
	ClearNotification();
	ClearWarning();
}

#undef LOCTEXT_NAMESPACE
