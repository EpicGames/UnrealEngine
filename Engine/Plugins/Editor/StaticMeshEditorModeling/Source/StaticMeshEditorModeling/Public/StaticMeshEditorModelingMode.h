// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Tools/Modes.h"
#include "StaticMeshEditorModelingMode.generated.h"

class SBorder;

UCLASS()
class UStaticMeshEditorModelingMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:

	const static FEditorModeID Id;

protected:

	UStaticMeshEditorModelingMode();

	virtual void Enter() override;
	virtual bool OnRequestClose() override;

	virtual bool UsesToolkits() const override;
	virtual void CreateToolkit() override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
};
