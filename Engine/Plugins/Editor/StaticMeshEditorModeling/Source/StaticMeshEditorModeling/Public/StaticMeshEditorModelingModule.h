// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IStaticMeshEditor;
class SDockTab;

class FStaticMeshEditorModelingModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	bool IsStaticMeshEditorModelingModeActive(TWeakPtr<IStaticMeshEditor> InEditor) const;
	void OnToggleStaticMeshEditorModelingMode(TWeakPtr<IStaticMeshEditor> InEditor);

	void RegisterMenusAndToolbars();

private:

	TWeakPtr<SDockTab> MeshLODDockTab;

};

