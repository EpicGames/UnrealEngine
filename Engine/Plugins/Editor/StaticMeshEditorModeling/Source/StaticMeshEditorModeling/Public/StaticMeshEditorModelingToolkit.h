// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"

class STextBlock;

class FStaticMeshEditorModelingToolkit : public FModeToolkit
{
public:

	void Init( const TSharedPtr<IToolkitHost>& InToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode ) override;

	FName GetToolkitFName() const override;
	FText GetBaseToolkitName() const override;
	TSharedPtr<SWidget> GetInlineContent() const override;

	void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
	FText GetToolPaletteDisplayName(FName PaletteName) const override;
	void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder) override;

	void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

private:

	void PostNotification(const FText& Message);
	void ClearNotification();
	void PostWarning(const FText& Message);
	void ClearWarning();

	void UpdateActiveToolProperties(UInteractiveTool* Tool);

	FText GetActiveToolDisplayName() const override;
	FText GetActiveToolMessage() const override;

	TSharedPtr<SBorder> ToolDetailsContainer;

	TSharedPtr<SWidget> ViewportOverlayWidget;
	const FSlateBrush* ActiveToolIcon = nullptr;
	FText ActiveToolName;
	FText ActiveToolMessage;

	TSharedPtr<STextBlock> ModeWarningArea;
	TSharedPtr<STextBlock> ModeHeaderArea;
	TSharedPtr<STextBlock> ToolWarningArea;

};



