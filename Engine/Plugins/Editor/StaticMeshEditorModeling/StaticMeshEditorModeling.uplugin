{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Static Mesh Editor Modeling Mode",
	"Description": "Enable a Modeling Tools Tab in the Static Mesh Editor",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Hidden": false,
	"Modules": [
		{
			"Name": "StaticMeshEditorModeling",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
	],
	"Plugins": [
		{
			"Name": "MeshLODToolset",
			"Enabled": true
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true
		}
	]
}
